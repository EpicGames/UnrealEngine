// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamingNetworkPlatformFile.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "HAL/IPlatformFileModule.h"
#include "Templates/UniquePtr.h"

DEFINE_LOG_CATEGORY(LogStreamingPlatformFile);


/**
 * A helper class for wrapping some of the network file payload specifics
 */
class FStreamingNetworkFileArchive
	: public FBufferArchive
{
public:

	FStreamingNetworkFileArchive(uint32 Command)
	{
		// make sure the command is at the start
		*this << Command;
	}

	// helper to serialize TCHAR* (there are a lot)
	FORCEINLINE friend FStreamingNetworkFileArchive& operator<<(FStreamingNetworkFileArchive& Ar, const TCHAR*& Str)
	{
		FString Temp(Str);
		Ar << Temp;
		return Ar;
	}
};


static const int32 GBufferCacheSize = 64 * 1024;


class FStreamingNetworkFileHandle
	: public IFileHandle
{
	FStreamingNetworkPlatformFile& Network;
	FString Filename;
	uint64 HandleId;
	int64 FilePos;
	int64 FileSize;
	bool bWritable;
	bool bReadable;
	uint8 BufferCache[2][GBufferCacheSize];
	int64 CacheStart[2];
	int64 CacheEnd[2];
	bool LazySeek;
	int32 CurrentCache;

public:

	FStreamingNetworkFileHandle(FStreamingNetworkPlatformFile& InNetwork, const TCHAR* InFilename, uint64 InHandleId, int64 InFileSize, bool bWriting)
		: Network(InNetwork)
		, Filename(InFilename)
		, HandleId(InHandleId)
		, FilePos(0)
		, FileSize(InFileSize)
		, bWritable(bWriting)
		, bReadable(!bWriting)
		, LazySeek(false)
		,CurrentCache(0)
	{
		CacheStart[0] = CacheStart[1] = -1;
		CacheEnd[0] = CacheEnd[1] = -1;
	}

	~FStreamingNetworkFileHandle()
	{
		Network.SendCloseMessage(HandleId);
	}

	virtual int64 Size() override
	{
		return FileSize;
	}

	virtual int64 Tell() override
	{
		return FilePos;
	}

	virtual bool Seek(int64 NewPosition) override
	{
		if( bWritable )
		{
			if( NewPosition == FilePos )
			{
				return true;
			}

			if (NewPosition >= 0 && NewPosition <= FileSize)
			{
				if (Network.SendSeekMessage(HandleId, NewPosition))
				{
					FilePos = NewPosition;

					return true;
				}
			}
		}
		else if( bReadable )
		{
			if (NewPosition >= 0 && NewPosition <= FileSize)
			{
				if (NewPosition < CacheStart[0] || NewPosition >= CacheEnd[0] || CacheStart[0] == -1)
				{
					if (NewPosition < CacheStart[1] || NewPosition >= CacheEnd[1] || CacheStart[1] == -1)
					{
						LazySeek = true;
						FilePos = NewPosition;
						if (CacheStart[CurrentCache] != -1)
						{
							CurrentCache++;
							CurrentCache %= 2;
							CacheStart[CurrentCache] = -1; // Invalidate the cache
						}

						return true;
					}
					else
					{
						LazySeek = false;
						FilePos = NewPosition;
						CurrentCache = 1;

						return true;
					}
				}
				else
				{
					LazySeek = false;
					FilePos = NewPosition;
					CurrentCache = 0;

					return true;
				}
			}
		}

		return false;
	}

	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		return Seek(FileSize + NewPositionRelativeToEnd);
	}

	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		bool Result = false;
		if (bReadable && BytesToRead >= 0 && BytesToRead + FilePos <= FileSize)
		{
			if (BytesToRead == 0)
			{
				Result = true;
			}
			else
			{
				if (BytesToRead > GBufferCacheSize) // reading more than we cache
				{
					// if the file position is within the cache, copy out the remainder of the cache
					if (CacheStart[CurrentCache] != -1 && FilePos >= CacheStart[CurrentCache] && FilePos < CacheEnd[CurrentCache])
					{
						int64 CopyBytes = CacheEnd[CurrentCache]-FilePos;
						FMemory::Memcpy(Destination, BufferCache[CurrentCache]+(FilePos-CacheStart[CurrentCache]), CopyBytes);
						FilePos += CopyBytes;
						BytesToRead -= CopyBytes;
						Destination += CopyBytes;
					}

					if (Network.SendSeekMessage(HandleId, FilePos))
					{
						Result = Network.SendReadMessage(HandleId, Destination, BytesToRead);
					}
					if (Result)
					{
						FilePos += BytesToRead;
						CurrentCache++;
						CurrentCache %= 2;
						CacheStart[CurrentCache] = -1; // Invalidate the cache
					}
				}
				else
				{
					Result = true;

					// need to update the cache
					if (CacheStart[CurrentCache] == -1 && FileSize < GBufferCacheSize)
					{
						Result = Network.SendReadMessage(HandleId, BufferCache[CurrentCache], FileSize);
						if (Result)
						{
							CacheStart[CurrentCache] = 0;
							CacheEnd[CurrentCache] = FileSize;
						}
					}
					else if (FilePos + BytesToRead > CacheEnd[CurrentCache] || CacheStart[CurrentCache] == -1 || FilePos < CacheStart[CurrentCache])
					{
						// copy the data from FilePos to the end of the Cache to the destination as long as it is in the cache
						if (CacheStart[CurrentCache] != -1 && FilePos >= CacheStart[CurrentCache] && FilePos < CacheEnd[CurrentCache])
						{
							int64 CopyBytes = CacheEnd[CurrentCache]-FilePos;
							FMemory::Memcpy(Destination, BufferCache[CurrentCache]+(FilePos-CacheStart[CurrentCache]), CopyBytes);
							FilePos += CopyBytes;
							BytesToRead -= CopyBytes;
							Destination += CopyBytes;
						}

						// switch to the other cache
						if (CacheStart[CurrentCache] != -1)
						{
							CurrentCache++;
							CurrentCache %= 2;
						}

						int64 SizeToRead = GBufferCacheSize;

						if (FilePos + SizeToRead > FileSize)
						{
							SizeToRead = FileSize-FilePos;
						}

						if (Network.SendSeekMessage(HandleId, FilePos))
						{
							Result = Network.SendReadMessage(HandleId, BufferCache[CurrentCache], SizeToRead);
						}

						if (Result)
						{
							CacheStart[CurrentCache] = FilePos;
							CacheEnd[CurrentCache] = FilePos + SizeToRead;
						}
					}

					// copy from the cache to the destination
					if (Result)
					{
						FMemory::Memcpy(Destination, BufferCache[CurrentCache]+(FilePos-CacheStart[CurrentCache]), BytesToRead);
						FilePos += BytesToRead;
					}
				}
			}
		}

		return Result;
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		if (!bReadable || BytesToRead < 0 || Offset < 0 || (BytesToRead + Offset) > Size())
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		return Network.SendReadMessage(HandleId, Destination, BytesToRead, Offset);
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		bool Result = false;

		if (bWritable && BytesToWrite >= 0)
		{
			if (BytesToWrite == 0)
			{
				Result = true;
			}
			else
			{
				Result = Network.SendWriteMessage(HandleId, Source, BytesToWrite);

				if (Result)
				{
					FilePos += BytesToWrite;
					FileSize = FMath::Max<int64>(FilePos, FileSize);
				}
			}
		}

		return Result;
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
		return false;
	}

	virtual bool Truncate(int64 NewSize) override
	{
		return false;
	}
};


bool FStreamingNetworkPlatformFile::ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const
{
	bool bResult = FNetworkPlatformFile::ShouldBeUsed(Inner, CmdLine);

	if (bResult)
	{
		bResult = FParse::Param(CmdLine, TEXT("Streaming")) || !FPlatformMisc::SupportsLocalCaching();
		if (FPlatformMisc::AllowLocalCaching())
		{
			// On desktop platforms, assume 'in place' execution - to prevent deletion of content, force streaming,
			// unless it's explicitly allowed with -allowcaching (for automation tests with staged builds).
			const bool bAllowCaching = FParse::Param(CmdLine, TEXT("AllowCaching"));

			if (bResult == false && !bAllowCaching)
			{
				UE_LOGF(LogStreamingPlatformFile, Display, "Platform supports local caching, but requires explicitly specifying -AllowCaching to use it.");
			}
			bResult = bResult || !bAllowCaching;
		}
		UE_CLOGF(bResult, LogStreamingPlatformFile, Display, "Using streaming network file system.");
	}

	return bResult;
}


bool FStreamingNetworkPlatformFile::InitializeInternal(IPlatformFile* Inner, const TCHAR* HostIP)
{
	// look for the commandline that will read files from over the network
	if (HostIP == nullptr)
	{
		UE_LOGF(LogStreamingPlatformFile, Error, "No Host IP specified in the commandline.");
		bIsUsable = false;

		return false;
	}

	// optionally get the port from the command line
	int32 OverridePort;
	if (FParse::Value(FCommandLine::Get(), TEXT("fileserverport="), OverridePort))
	{
		UE_LOGF(LogStreamingPlatformFile, Display, "Overriding file server port: %d", OverridePort);
		FileServerPort = OverridePort;
	}

	// Send the filenames and timestamps to the server.
	FNetworkFileArchive Payload(NFS_Messages::GetFileList);
	FillGetFileList(Payload);

	// Send the directories over, and wait for a response.
	FArrayReader Response;

	if(SendPayloadAndReceiveResponse(Payload,Response))
	{
		// Receive the cooked version information.
		FPackageFileVersion ServerPackageVersion;
		int32 ServerPackageLicenseeVersion = 0;
		ProcessServerInitialResponse(Response, ServerPackageVersion, ServerPackageLicenseeVersion);

		// Make sure we can sync a file.
		FString TestSyncFile = FPaths::Combine(*(FPaths::EngineDir()), TEXT("Config/BaseEngine.ini"));
		IFileHandle* TestFileHandle = OpenRead(*TestSyncFile);
		if (TestFileHandle != nullptr)
		{
			uint8* FileContents = (uint8*)FMemory::Malloc(TestFileHandle->Size());
			if (!TestFileHandle->Read(FileContents, TestFileHandle->Size()))
			{
				UE_LOGF(LogStreamingPlatformFile, Fatal, "Could not read test file %ls.", *TestSyncFile);
			}
			FMemory::Free(FileContents);
			delete TestFileHandle;
		}
		else
		{
			UE_LOGF(LogStreamingPlatformFile, Fatal, "Could not open test file %ls.", *TestSyncFile);
		}

		FCommandLine::AddToSubprocessCommandLine( *FString::Printf( TEXT("-StreamingHostIP=%s"), HostIP ), ECommandLineArgumentFlags::AllContexts );

		return true; 
	}

	return false; 
}


FStreamingNetworkPlatformFile::~FStreamingNetworkPlatformFile()
{
}


bool FStreamingNetworkPlatformFile::DeleteFile(const TCHAR* Filename)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);

	FStreamingNetworkFileArchive Payload(NFS_Messages::DeleteFile);
	Payload << RelativeFilename;

	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	uint32 Success = 0;
	Response << Success;

	return !!Success;
}


bool FStreamingNetworkPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	FFileInfo Info;
	GetFileInfo(Filename, Info);

	return Info.ReadOnly;
}


bool FStreamingNetworkPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FString RelativeFrom = From; 
	MakeStandardNetworkFilename(RelativeFrom);
	FString RelativeTo = To; 
	MakeStandardNetworkFilename(RelativeTo);

	FStreamingNetworkFileArchive Payload(NFS_Messages::MoveFile);
	Payload << RelativeFrom;
	Payload << RelativeTo;

	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	uint32 Success = 0;
	Response << Success;

	return !!Success;
}


bool FStreamingNetworkPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::SetReadOnly);
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);

	Payload << RelativeFilename;
	Payload << bNewReadOnlyValue;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	bool bSuccess = 0;
	Response << bSuccess;

	return bSuccess;
}


FDateTime FStreamingNetworkPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	FFileInfo Info;
	GetFileInfo(Filename, Info);

	return Info.TimeStamp;
}


void FStreamingNetworkPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::SetTimeStamp);
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);
	Payload << RelativeFilename;
	Payload << DateTime;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return;
	}

	bool bSuccess = 0;
	Response << bSuccess;
}


FDateTime FStreamingNetworkPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	FFileInfo Info;
	GetFileInfo(Filename, Info);

	return Info.AccessTimeStamp;
}


IFileHandle* FStreamingNetworkPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);

	FStreamingNetworkFileHandle* FileHandle = SendOpenMessage(RelativeFilename, false, false, false);

	return FileHandle;
}


IFileHandle* FStreamingNetworkPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);
	FStreamingNetworkFileHandle* FileHandle = SendOpenMessage(RelativeFilename, true, bAppend, bAllowRead);

	return FileHandle;
}

bool FStreamingNetworkPlatformFile::CreateDirectoryTree(const TCHAR* Directory)
{
	return IPlatformFile::CreateDirectoryTree( Directory );
}


bool FStreamingNetworkPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::CreateDirectory);
	FString RelativeDirectory = Directory;
	MakeStandardNetworkFilename(RelativeDirectory);
	Payload << RelativeDirectory;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	bool bSuccess = 0;
	Response << bSuccess;

	return bSuccess;
}


bool FStreamingNetworkPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::DeleteDirectory);
	FString RelativeDirectory = Directory;
	MakeStandardNetworkFilename(RelativeDirectory);
	Payload << RelativeDirectory;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	bool bSuccess = 0;
	Response << bSuccess;

	return bSuccess;
}


bool FStreamingNetworkPlatformFile::IterateDirectory(const TCHAR* InDirectory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	FString RelativeDirectory = InDirectory;
	MakeStandardNetworkFilename(RelativeDirectory);
	// for .dll, etc searches that don't specify a path, we need to strip off the path
	// before we send it to the visitor
	bool bHadNoPath = InDirectory[0] == 0;

	// we loop until this is false
	bool RetVal = true;

	// Find the directory in TOC
	FServerTOC::FDirectory* ServerDirectory = ServerFiles.FindDirectory(RelativeDirectory);
	if (ServerDirectory != nullptr)
	{
		// loop over the server files and look if they are in this exact directory
		for (FServerTOC::FDirectory::TIterator It(*ServerDirectory); It && RetVal == true; ++It)
		{
			if (FPaths::GetPath(It.Key()) == RelativeDirectory)
			{
				// timestamps of 0 mean directories
				bool bIsDirectory = It.Value() == 0;
			
				// visit (stripping off the path if needed)
				RetVal = Visitor.CallShouldVisitAndVisit(bHadNoPath ? *FPaths::GetCleanFilename(It.Key()) : *It.Key(), bIsDirectory);
			}
		}
	}

	return RetVal;
}


bool FStreamingNetworkPlatformFile::IterateDirectoryRecursively(const TCHAR* InDirectory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	FString RelativeDirectory = InDirectory;
	MakeStandardNetworkFilename(RelativeDirectory);
	// we loop until this is false
	bool RetVal = true;

	// loop over the server TOC
	for (TMap<FString, FServerTOC::FDirectory*>::TIterator DirIt(ServerFiles.Directories); DirIt && RetVal == true; ++DirIt)
	{
		if (DirIt.Key().StartsWith(RelativeDirectory))
		{
			FServerTOC::FDirectory& ServerDirectory = *DirIt.Value();
		
			// loop over the server files and look if they are in this exact directory
			for (FServerTOC::FDirectory::TIterator It(ServerDirectory); It && RetVal == true; ++It)
			{
				// timestamps of 0 mean directories
				bool bIsDirectory = It.Value() == 0;

				// visit!
				RetVal = Visitor.CallShouldVisitAndVisit(*It.Key(), bIsDirectory);
			}
		}
	}

	return RetVal;
}


bool FStreamingNetworkPlatformFile::DeleteDirectoryRecursively(const TCHAR* Directory)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::DeleteDirectoryRecursively);
	FString RelativeDirectory = Directory;
	MakeStandardNetworkFilename(RelativeDirectory);
	Payload << RelativeDirectory;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	bool bSuccess = 0;
	Response << bSuccess;

	return bSuccess;
}


bool FStreamingNetworkPlatformFile::CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags, EPlatformFileWrite WriteFlags)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::CopyFile);
	FString RelativeTo = To; MakeStandardNetworkFilename(RelativeTo);
	FString RelativeFrom = From; MakeStandardNetworkFilename(RelativeFrom);
	Payload << RelativeTo;
	Payload << RelativeFrom;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	bool bSuccess = 0;
	Response << bSuccess;

	return bSuccess;
}


FString FStreamingNetworkPlatformFile::ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename )
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::ToAbsolutePathForRead);
	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);
	Payload << RelativeFilename;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == true)
	{
		Response << RelativeFilename;
	}

	return RelativeFilename;
}


FString FStreamingNetworkPlatformFile::ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename )
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FStreamingNetworkFileArchive Payload(NFS_Messages::ToAbsolutePathForWrite);
	FString RelativeFilename = Filename; 
	MakeStandardNetworkFilename(RelativeFilename);
	Payload << RelativeFilename;

	// perform a local operation
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == true)
	{
		Response << RelativeFilename;
	}

	return RelativeFilename;
}


bool FStreamingNetworkPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	// If there are any syncable files in this directory, consider it existing
	FString RelativeDirectory = Directory;
	MakeStandardNetworkFilename(RelativeDirectory);
	FServerTOC::FDirectory* ServerDirectory = ServerFiles.FindDirectory(RelativeDirectory);

	return ServerDirectory != nullptr;
}


void FStreamingNetworkPlatformFile::GetFileInfo(const TCHAR* Filename, FFileInfo& Info)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	FString RelativeFilename = Filename;
	MakeStandardNetworkFilename(RelativeFilename);

	FStreamingNetworkFileArchive Payload(NFS_Messages::GetFileInfo);
	Payload << const_cast<FString&>(RelativeFilename);

	// Send the filename over
	FArrayReader Response;
	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return;
	}

	// Get info from the response
	Response << Info.FileExists;
	Response << Info.ReadOnly;
	Response << Info.Size;
	Response << Info.TimeStamp;
	Response << Info.AccessTimeStamp;
}


FStreamingNetworkFileHandle* FStreamingNetworkPlatformFile::SendOpenMessage(const FString& Filename, bool bIsWriting, bool bAppend, bool bAllowRead)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	
	FStreamingNetworkFileArchive Payload(bIsWriting ? NFS_Messages::OpenWrite : NFS_Messages::OpenRead);
	Payload << const_cast<FString&>(Filename);

	if (bIsWriting)
	{
		Payload << bAppend;
		Payload << bAllowRead;
	}

	// Send the filename over
	FArrayReader Response;
	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return nullptr;
	}

	// This server handle ID which will be used to perform operations on this file.
	uint64 HandleId = 0;
	Response << HandleId;

	// Get the server file timestamp
	FDateTime ServerTimeStamp;
	Response << ServerTimeStamp;

	// Get the server file size
	int64 ServerFileSize = 0;
	Response << ServerFileSize;

	if (bIsWriting || ServerFileSize > 0)
	{
		FStreamingNetworkFileHandle* FileHandle = new FStreamingNetworkFileHandle(*this, *Filename, HandleId, ServerFileSize, bIsWriting);
		return FileHandle;
	}
	else
	{
		return nullptr;
	}
}


bool FStreamingNetworkPlatformFile::SendReadMessage(uint64 HandleId, uint8* Destination, int64 BytesToRead)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	// Send the filename over.
	FStreamingNetworkFileArchive Payload(NFS_Messages::Read);
	Payload << HandleId;
	Payload << BytesToRead;

	// Send the filename over
	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	// Get the server number of bytes read.
	int64 ServerBytesRead = 0;
	Response << ServerBytesRead;

	bool bSuccess = (ServerBytesRead == BytesToRead);

	if (bSuccess)
	{
		// Get the data.
		Response.Serialize(Destination, BytesToRead);
	}

	return bSuccess;
}


STREAMINGFILE_API bool FStreamingNetworkPlatformFile::SendReadMessage(uint64 HandleId, uint8* Destination, int64 BytesToRead, int64 Offset)
{
	FStreamingNetworkFileArchive Payload(NFS_Messages::ReadAt);
	Payload << HandleId;
	Payload << BytesToRead;
	Payload << Offset;

	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	int64 ServerBytesRead = 0;
	Response << ServerBytesRead;

	if (ServerBytesRead == BytesToRead)
	{
		Response.Serialize(Destination, BytesToRead);
		return true;
	}
	else
	{
		return false;
	}
}

bool FStreamingNetworkPlatformFile::SendWriteMessage(uint64 HandleId, const uint8* Source, int64 BytesToWrite)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	// Send the filename over.
	FStreamingNetworkFileArchive Payload(NFS_Messages::Write);
	Payload << HandleId;
	// Send the data over
	Payload << BytesToWrite;
	Payload.Serialize(const_cast<uint8*>(Source), BytesToWrite);

	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false) 
	{
		return false;
	}

	// Get the number of bytes the server wrote.
	int64 ServerBytesWritten = 0;
	Response << ServerBytesWritten;

	return (ServerBytesWritten == BytesToWrite);
}


bool FStreamingNetworkPlatformFile::SendSeekMessage(uint64 HandleId, int64 NewPosition)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	// Send the filename over.
	FStreamingNetworkFileArchive Payload(NFS_Messages::Seek);
	Payload << HandleId;
	Payload << NewPosition;

	FArrayReader Response;

	if (SendPayloadAndReceiveResponse(Payload, Response) == false)
	{
		return false;
	}

	int64 ServerNewPosition = -1;
	Response << ServerNewPosition;

	return (ServerNewPosition == NewPosition);
}


bool FStreamingNetworkPlatformFile::SendCloseMessage(uint64 HandleId)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	// Send the filename over (cast away const here because we know this << will not modify the string)
	FStreamingNetworkFileArchive Payload(NFS_Messages::Close);
	Payload << HandleId;

	FArrayReader Response;

	return SendPayloadAndReceiveResponse(Payload, Response);
}


void FStreamingNetworkPlatformFile::PerformHeartbeat()
{
	FNetworkFileArchive Payload(NFS_Messages::Heartbeat);

	// send the filename over
	FArrayReader Response;

	if(SendPayloadAndReceiveResponse(Payload,Response))
	{
		return;
	}
  
    check(0);
}


/**
 * Module for the streaming file
 */
class FStreamingFileModule
	: public IPlatformFileModule
{
public:

	virtual IPlatformFile* GetPlatformFile() override
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton = MakeUnique<FStreamingNetworkPlatformFile>();
		return AutoDestroySingleton.Get();
	}
};


IMPLEMENT_MODULE(FStreamingFileModule, StreamingFile);
