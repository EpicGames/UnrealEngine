// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/Paths.h"
#include "NetworkMessage.h"
#include "NetworkPlatformFile.h"

DECLARE_LOG_CATEGORY_EXTERN(LogStreamingPlatformFile, Log, All);


#if 0
/**
 * Visitor to gather local files with their timestamps
 */
class FStreamingLocalTimestampVisitor : public IPlatformFile::FDirectoryVisitor
{
private:

	/** The file interface to use for any file operations */
	IPlatformFile& FileInterface;

	/** true if we want directories in this list */
	bool bCacheDirectories;

	/** A list of directories that we should not traverse into */
	TArray<FString> DirectoriesToIgnore;

	/** A list of directories that we should only go one level into */
	TArray<FString> DirectoriesToNotRecurse;

public:

	/** Relative paths to local files and their timestamps */
	TMap<FString, FDateTime> FileTimes;
	
	FStreamingLocalTimestampVisitor(IPlatformFile& InFileInterface, const TArray<FString>& InDirectoriesToIgnore, const TArray<FString>& InDirectoriesToNotRecurse, bool bInCacheDirectories=false)
		: FileInterface(InFileInterface)
		, bCacheDirectories(bInCacheDirectories)
	{
		// make sure the paths are standardized, since the Visitor will assume they are standard
		for (int32 DirIndex = 0; DirIndex < InDirectoriesToIgnore.Num(); DirIndex++)
		{
			FString DirToIgnore = InDirectoriesToIgnore[DirIndex];
			FPaths::MakeStandardFilename(DirToIgnore);
			DirectoriesToIgnore.Add(DirToIgnore);
		}

		for (int32 DirIndex = 0; DirIndex < InDirectoriesToNotRecurse.Num(); DirIndex++)
		{
			FString DirToNotRecurse = InDirectoriesToNotRecurse[DirIndex];
			FPaths::MakeStandardFilename(DirToNotRecurse);
			DirectoriesToNotRecurse.Add(DirToNotRecurse);
		}
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		// make sure all paths are "standardized" so the other end can match up with it's own standardized paths
		FString RelativeFilename = FilenameOrDirectory;
		FPaths::MakeStandardFilename(RelativeFilename);

		// cache files and optionally directories
		if (!bIsDirectory)
		{
			FileTimes.Add(RelativeFilename, FileInterface.GetTimeStamp(FilenameOrDirectory));
		}
		else if (bCacheDirectories)
		{
			// we use a timestamp of 0 to indicate a directory
			FileTimes.Add(RelativeFilename, 0);
		}

		// iterate over directories we care about
		if (bIsDirectory)
		{
			bool bShouldRecurse = true;
			// look in all the ignore directories looking for a match
			for (int32 DirIndex = 0; DirIndex < DirectoriesToIgnore.Num() && bShouldRecurse; DirIndex++)
			{
				if (RelativeFilename.StartsWith(DirectoriesToIgnore[DirIndex]))
				{
					bShouldRecurse = false;
				}
			}

			if (bShouldRecurse == true)
			{
				// If it is a directory that we should not recurse (ie we don't want to process subdirectories of it)
				// handle that case as well...
				for (int32 DirIndex = 0; DirIndex < DirectoriesToNotRecurse.Num() && bShouldRecurse; DirIndex++)
				{
					if (RelativeFilename.StartsWith(DirectoriesToNotRecurse[DirIndex]))
					{
						// Are we more than level deep in that directory?
						FString CheckFilename = RelativeFilename.Right(RelativeFilename.Len() - DirectoriesToNotRecurse[DirIndex].Len());
						if (CheckFilename.Len() > 1)
						{
							bShouldRecurse = false;
						}
					}
				}
			}

			// recurse if we should
			if (bShouldRecurse)
			{
				FileInterface.IterateDirectory(FilenameOrDirectory, *this);
			}
		}

		return true;
	}
};
#endif


/**
 * Wrapper to redirect the low level file system to a server
 */
class FStreamingNetworkPlatformFile
	: public FNetworkPlatformFile
{
	friend class FAsyncFileSync;

	// FNetworkPlatformFile interface
	STREAMINGFILE_API virtual bool InitializeInternal(IPlatformFile* Inner, const TCHAR* HostIP) override;

public:

	/** Default Constructor */
	FStreamingNetworkPlatformFile() 
	{
		HeartbeatFrequency = -1.0f;
		ConnectionFlags |= EConnectionFlags::Streaming;
	}

	/** Virtual destructor */
	STREAMINGFILE_API virtual ~FStreamingNetworkPlatformFile();

public:

	static const TCHAR* GetTypeName()
	{
		return TEXT("StreamingFile");
	}

	/** Sends Open message to the server and creates a new file handle if successful. */
	STREAMINGFILE_API class FStreamingNetworkFileHandle* SendOpenMessage(const FString& Filename, bool bIsWriting, bool bAppend, bool bAllowRead);

	/** Sends Read message to the server. */
	STREAMINGFILE_API bool SendReadMessage(uint64 HandleId, uint8* Destination, int64 BytesToRead);

	STREAMINGFILE_API bool SendReadMessage(uint64 HandleId, uint8* Destination, int64 BytesToRead, int64 Offset);

	/** Sends Write message to the server. */
	STREAMINGFILE_API bool SendWriteMessage(uint64 HandleId, const uint8* Source, int64 BytesToWrite);	

	/** Sends Seek message to the server. */
	STREAMINGFILE_API bool SendSeekMessage(uint64 HandleId, int64 NewPosition);	

	/** Sends Close message to the server. */
	STREAMINGFILE_API bool SendCloseMessage(uint64 HandleId);

public:

	// need to override what FNetworkPlatformFile does here
	void InitializeAfterSetActive() override { }

	// IPlatformFile interface

	STREAMINGFILE_API virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override;

	virtual IPlatformFile* GetLowerLevel() override
	{
		return nullptr;
	}
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		check(false);
	}

	virtual const TCHAR* GetName() const override
	{
		return FStreamingNetworkPlatformFile::GetTypeName();
	}

	STREAMINGFILE_API virtual bool DeleteFile(const TCHAR* Filename) override;
	STREAMINGFILE_API virtual bool IsReadOnly(const TCHAR* Filename) override;
	STREAMINGFILE_API virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	STREAMINGFILE_API virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	STREAMINGFILE_API virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	STREAMINGFILE_API virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	STREAMINGFILE_API virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	STREAMINGFILE_API virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	STREAMINGFILE_API virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead) override;
	STREAMINGFILE_API virtual bool DirectoryExists(const TCHAR* Directory) override;
	STREAMINGFILE_API virtual bool CreateDirectoryTree(const TCHAR* Directory) override;
	STREAMINGFILE_API virtual bool CreateDirectory(const TCHAR* Directory) override;
	STREAMINGFILE_API virtual bool DeleteDirectory(const TCHAR* Directory) override;
	STREAMINGFILE_API virtual bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	STREAMINGFILE_API virtual bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	STREAMINGFILE_API virtual bool DeleteDirectoryRecursively(const TCHAR* Directory) override;
	STREAMINGFILE_API virtual bool CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override;
	STREAMINGFILE_API virtual FString ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override;
	STREAMINGFILE_API virtual FString ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override;

private:

	// FNetworkPlatformFile interface

	STREAMINGFILE_API virtual void PerformHeartbeat() override;
	STREAMINGFILE_API virtual void GetFileInfo(const TCHAR* Filename, FFileInfo& Info) override;

private:

	/** Set files that the server said we should sync asynchronously */
	TArray<FString> FilesToSyncAsync;
};
