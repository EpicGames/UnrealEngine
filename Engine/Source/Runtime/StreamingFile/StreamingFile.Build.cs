// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StreamingFile : ModuleRules
{
	public StreamingFile(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "NetworkFile",
                "Sockets",
            }
        );
    }
}
